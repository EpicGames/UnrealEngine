// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "CharacterTrajectoryUAFFragments.generated.h"

// Variable name references used by UCharacterTrajectoryToUAFProcessor to write
// trajectory data to UAF components.
USTRUCT()
struct FCharacterTrajectoryUAFData : public FMassConstSharedFragment
{
	GENERATED_BODY()

	// The name of the trajectory variable to set on the UAF component
	UPROPERTY(EditAnywhere, Category = "UAF")
	FName PoseVariableName = TEXT("Trajectory");

	// The name of the target orientation variable to set on the UAF component
	UPROPERTY(EditAnywhere, Category = "UAF")
	FName SteeringVariableName = TEXT("TargetOrientation");

	// The name of the root world transform variable to set on the UAF component
	UPROPERTY(EditAnywhere, Category = "UAF")
	FName RootWorldTransformVariableName = TEXT("MeshWorldTransform");

	// The name of the delta transform variable to set on the UAF component
	UPROPERTY(EditAnywhere, Category = "UAF")
	FName DeltaTransformVariableName = TEXT("DeltaTransform");
};
