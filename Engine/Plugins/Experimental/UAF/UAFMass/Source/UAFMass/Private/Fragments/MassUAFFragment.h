// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonFragments.h"
#include "Module/SystemReference.h"
#include "UAF/UAFAssetData.h"
#include "MassUAFFragment.generated.h"


/**
 * Fragment owning the UAF system associated to the entity.
 */
USTRUCT()
struct FMassUAFFragment : public FObjectWrapperFragment
{
	GENERATED_BODY()
	// @TODO: We don't need this data after UAF system creation. Remove this when we figure out the correct init flow i.e. shared/chunk fragment or Subsystem ownership.
	UPROPERTY(transient)
	TInstancedStruct<FUAFSystemFactoryAsset> Asset;

	UE::UAF::FSystemReference SystemReference;
};

template<> 
struct TStructOpsTypeTraits<FMassUAFFragment> : public TStructOpsTypeTraitsBase2<FMassUAFFragment> 
{ 
	enum 
	{ 
		WithCopy = false 
	}; 
};

template<>
struct TMassFragmentTraits<FMassUAFFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};
