// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMTrait_ModuleEventDependency_MassPhaseProcessor.h"

#include "MassSimulationSubsystem.h"


#if WITH_EDITOR
FString FRigVMTrait_ModuleEventDependency_MassPhaseProcessor::GetDisplayName() const
{
	return StaticStruct()->GetDisplayNameText().ToString();
}
#endif // WITH_EDITOR

void FRigVMTrait_ModuleEventDependency_MassPhaseProcessor::OnAddDependency(const UE::UAF::FModuleDependencyContext& InContext) const
{
	check(IsInGameThread()); // TickFunction add / remove prerequisite is not thread safe
	UMassSimulationSubsystem* MassSimulationSubsystem = InContext.Object->GetWorld()->GetSubsystem<UMassSimulationSubsystem>();

	if (MassSimulationSubsystem == nullptr)
	{
		return;
	}

	FMassProcessingPhaseManager& MassSimulationPhaseManager = MassSimulationSubsystem->GetMutablePhaseManager(); 
	FTickFunction& ProcessingPhaseTickFunction = MassSimulationPhaseManager.GetProcessingPhaseTickFunction(DependentMassPhase);
	
	if (Ordering == EAnimNextModuleEventDependencyOrdering::Before)
	{
		InContext.TickFunction.AddPrerequisite(MassSimulationSubsystem, ProcessingPhaseTickFunction);
	}
	else
	{
		UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext.Object);
		ProcessingPhaseTickFunction.AddPrerequisite(AnimNextComponent, InContext.TickFunction);
	}
}


void FRigVMTrait_ModuleEventDependency_MassPhaseProcessor::OnRemoveDependency(const UE::UAF::FModuleDependencyContext& InContext) const
{
	check(IsInGameThread()); // TickFunction add / remove prerequisite is not thread safe
	UMassSimulationSubsystem* MassSimulationSubsystem = InContext.Object->GetWorld()->GetSubsystem<UMassSimulationSubsystem>();

	FMassProcessingPhaseManager& MassSimulationPhaseManager = MassSimulationSubsystem->GetMutablePhaseManager(); 
	FTickFunction& ProcessingPhaseTickFunction = MassSimulationPhaseManager.GetProcessingPhaseTickFunction(DependentMassPhase);
	
	if (Ordering == EAnimNextModuleEventDependencyOrdering::Before)
	{
		InContext.TickFunction.RemovePrerequisite(MassSimulationSubsystem, ProcessingPhaseTickFunction);
	}
	else
	{
		UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext.Object);
		ProcessingPhaseTickFunction.RemovePrerequisite(AnimNextComponent, InContext.TickFunction);
	}
}