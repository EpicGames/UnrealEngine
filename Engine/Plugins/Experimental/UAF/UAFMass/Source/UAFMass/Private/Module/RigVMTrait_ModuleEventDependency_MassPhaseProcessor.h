// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessingTypes.h"
#include "Module/RigVMTrait_ModuleEventDependency.h"
#include "RigVMTrait_ModuleEventDependency_MassPhaseProcessor.generated.h"


/** A dependency on one of a mass phase processors (equates to a tick function)
 */
USTRUCT(DisplayName = "Mass Phase Processor", meta = (ShowTooltip = true))
struct FRigVMTrait_ModuleEventDependency_MassPhaseProcessor : public FRigVMTrait_ModuleEventDependency
{
	GENERATED_BODY()

	// FRigVMTrait interface
#if WITH_EDITOR
	virtual FString GetDisplayName() const override;
#endif

	// FAnimNextModuleEventDependency interface
	virtual void OnAddDependency(const UE::UAF::FModuleDependencyContext& InContext) const override;
	virtual void OnRemoveDependency(const UE::UAF::FModuleDependencyContext& InContext) const override;


	// The Mass processing phase that this dependency relates to
	UPROPERTY(EditAnywhere, Category = "Dependency")
	EMassProcessingPhase DependentMassPhase = EMassProcessingPhase::PrePhysics;
};