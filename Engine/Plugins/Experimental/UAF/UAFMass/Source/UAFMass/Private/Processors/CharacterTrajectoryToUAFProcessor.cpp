// Copyright Epic Games, Inc. All Rights Reserved.

#include "CharacterTrajectoryToUAFProcessor.h"

#include "Fragments/MassUAFComponentFragment.h"
#include "Fragments/CharacterTrajectoryUAFFragments.h"
#include "MassExecutionContext.h"
#include "Component/AnimNextComponent.h"
#include "MassCharacterTrajectoryFragments.h"
#include "Variables/AnimNextVariableReference.h"

UCharacterTrajectoryToUAFProcessor::UCharacterTrajectoryToUAFProcessor() : EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::CharacterTrajectoryToUAF;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::CharacterTrajectoryCollision);
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::CharacterTrajectoryGeneration);
}

void UCharacterTrajectoryToUAFProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FCharacterTrajectoryFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassUAFComponentWrapperFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddConstSharedRequirement<FCharacterTrajectoryUAFData>(EMassFragmentPresence::All);
}

void UCharacterTrajectoryToUAFProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ParallelForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			const float DeltaTime = Context.GetDeltaTimeSeconds();
			const int32 NumEntities = Context.GetNumEntities();
			const FCharacterTrajectoryUAFData& UAFData = Context.GetConstSharedFragment<FCharacterTrajectoryUAFData>();

			const TConstArrayView<FMassUAFComponentWrapperFragment> UAFComponentList = Context.GetFragmentView<FMassUAFComponentWrapperFragment>();
			const TConstArrayView<FCharacterTrajectoryFragment> TrajectoryList = Context.GetFragmentView<FCharacterTrajectoryFragment>();

			for (int32 i = 0; i < NumEntities; i++)
			{
				// Write trajectory info to UAF component
				// Do not pin since expensive. Assume the start to end execution of mass is within a frame & thus GC is not a concern.
				if (UUAFComponent* Component = UAFComponentList[i].Component.Get())
				{
					FTransformTrajectorySample PrevSample = TrajectoryList[i].Trajectory.GetSampleAtTime(0.0f);
					FTransformTrajectorySample CurrentSample = TrajectoryList[i].Trajectory.GetSampleAtTime(DeltaTime);

					FTransform DeltaTransform = CurrentSample.GetTransform() * PrevSample.GetTransform().Inverse();

					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					Component->SetVariable(FAnimNextVariableReference(UAFData.PoseVariableName), TrajectoryList[i].Trajectory);
					Component->SetVariable(FAnimNextVariableReference(UAFData.SteeringVariableName), TrajectoryList[i].SteeringTarget);
					Component->SetVariable(FAnimNextVariableReference(UAFData.RootWorldTransformVariableName),
						TrajectoryList[i].MeshRootWorldTransform);
					Component->SetVariable(FAnimNextVariableReference(UAFData.DeltaTransformVariableName),
						DeltaTransform);
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}
		});
}
