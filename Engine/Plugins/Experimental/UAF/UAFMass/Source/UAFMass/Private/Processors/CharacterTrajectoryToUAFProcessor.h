// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassProcessor.h"
#include "MassCharacterTrajectoryTypes.h"

#include "CharacterTrajectoryToUAFProcessor.generated.h"

namespace UE::Mass::ProcessorGroupNames
{
inline const FName CharacterTrajectoryToUAF = FName(TEXT("CharacterTrajectoryToUAF"));
} // namespace UE::Mass::ProcessorGroupNames

UCLASS()
class UCharacterTrajectoryToUAFProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UCharacterTrajectoryToUAFProcessor();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};
