// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassUAFProcessor.h"

#include "Fragments/MassUAFFragment.h"
#include "MassExecutionContext.h"
#include "MassCommonTypes.h"
#include "Module/ModuleTickFunction.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "Subsystems/MassUAFSubsystem.h"

DEFINE_LOG_CATEGORY(LogMassUAF);

namespace UE::Mass::ProcessorGroupNames
{
	static const FName Animation = FName(TEXT("Animation"));
	static const FName AnimationDebug = FName(TEXT("AnimationDebug"));
} // namespace UE::Mass::ProcessorGroupNames

namespace UE::UAF::Mass
{
	static bool bDebug = false;
	FAutoConsoleVariableRef CVarDebugAnimPose(TEXT("Mass.UAF.Debug"), bDebug, TEXT(""));
}

UMassUAFInitializer::UMassUAFInitializer()
	: EntityQuery(*this)
{
	ObservedTypes.Add(FMassUAFFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Add;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	bRequiresGameThreadExecution = false;
}

void UMassUAFInitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassUAFFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassUAFSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassUAFInitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	using namespace UE::UAF;	
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		FTaskTagScope Scope(ETaskTag::EParallelGameThread);

		UMassUAFSubsystem& UAFSubsystem = Context.GetMutableSubsystemChecked<UMassUAFSubsystem>();
		const int32 NumEntities = Context.GetNumEntities();
		TArrayView<FMassUAFFragment> UAFFragmentList = Context.GetMutableFragmentView<FMassUAFFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassUAFFragment& UAFFragment = UAFFragmentList[EntityIt];

			if (UAFFragment.Asset.IsValid())
			{
				// @TODO: Passing a dummy UAFSubsystem since it works well with rewind debugger + we need a UObject owner, but should go away once we agree on system ownership.
				UAFFragment.SystemReference = FSystemReference(UAFFragment.Asset, &UAFSubsystem, EAnimNextModuleInitMethod::None);
			}
		}
	});
}

UMassUAFProcessor::UMassUAFProcessor() : EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Animation;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Movement);
	bRequiresGameThreadExecution = false;
}

void UMassUAFProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassUAFFragment>(EMassFragmentAccess::ReadOnly);
}

void UMassUAFProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	using namespace UE::UAF;	
	EntityQuery.ParallelForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		FTaskTagScope Scope(ETaskTag::EParallelGameThread);
		const int32 NumEntities = Context.GetNumEntities();
		const TConstArrayView<FMassUAFFragment> UAFFragmentList = Context.GetFragmentView<FMassUAFFragment>();
		
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassEntityHandle Entity = Context.GetEntity(EntityIt);
			const FMassUAFFragment& UAFFragment = UAFFragmentList[EntityIt];

			if (UAFFragment.SystemReference.IsValid())
			{
				// @TODO: Pass Mass execution context to UAF update
				//SystemReference->QueueTask(NAME_None, [&Context, Entity, &UAFFragment](const UE::UAF::FModuleTaskContext& InContext)
				//{
				//How do we do this? There's no ExecuteContext accessible here.
				//});

				UAFFragment.SystemReference.RunEvent(FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName, Context.GetDeltaTimeSeconds());
			}
		}
	});
}

UMassAnimPoseDebugProcessor::UMassAnimPoseDebugProcessor() : EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::AnimationDebug;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Animation);
}

void UMassAnimPoseDebugProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddSubsystemRequirement<UMassUAFSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassAnimPoseDebugProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	UWorld* World = GetWorld();
	if (World == nullptr || (UE::UAF::Mass::bDebug == false))
	{
		return;
	}

	// @TODO: Update this when Mass has a way to access UAF's pose buffer.
	/*using namespace UE::UAF;	
	EntityQuery.ParallelForEachEntityChunk(Context, [this, World](FMassExecutionContext& Context)
	{
		UMassUAFSubsystem& UAFSubsystem = Context.GetMutableSubsystemChecked<UMassUAFSubsystem>();

		const int32 NumEntities = Context.GetNumEntities();
		const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
		
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
		}
	});*/
}

UMassUAFDestructor::UMassUAFDestructor()
	: EntityQuery(*this)
{
	ObservedTypes.Add(FMassUAFFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	bRequiresGameThreadExecution = false;
}

void UMassUAFDestructor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassUAFFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassUAFDestructor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	using namespace UE::UAF;
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			FTaskTagScope Scope(ETaskTag::EParallelGameThread);

			const int32 NumEntities = Context.GetNumEntities();
			TArrayView<FMassUAFFragment> UAFFragmentList = Context.GetMutableFragmentView<FMassUAFFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				FMassUAFFragment& UAFFragment = UAFFragmentList[EntityIt];
				UAFFragment.SystemReference.Reset();
			}
		});
}