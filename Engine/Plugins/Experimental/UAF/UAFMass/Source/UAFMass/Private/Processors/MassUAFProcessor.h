// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassProcessor.h"
#include "MassObserverProcessor.h"

#include "MassUAFProcessor.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMassUAF, Display, All);

UCLASS()
class UMassUAFInitializer : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UMassUAFInitializer();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};


UCLASS()
class UMassUAFProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UMassUAFProcessor();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

UCLASS()
class UMassAnimPoseDebugProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UMassAnimPoseDebugProcessor();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

UCLASS()
class UMassUAFDestructor : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UMassUAFDestructor();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};