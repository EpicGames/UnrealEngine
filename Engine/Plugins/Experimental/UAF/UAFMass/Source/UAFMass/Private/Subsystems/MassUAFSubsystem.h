// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityHandle.h"
#include "MassEntityTypes.h"
#include "MassSubsystemBase.h"
#include "MassUAFSubsystem.generated.h"

/** @TODO: Dummy subsystem used as UObject owner of Mass - owned UAF systems.
 * Remove this when we have proper ownership of UAF system + per-entity rewind debugger support. 
 */
UCLASS(MinimalAPI)
class UMassUAFSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()	
};

template<>
struct TMassExternalSubsystemTraits<UMassUAFSubsystem> final
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = true
	};
};
