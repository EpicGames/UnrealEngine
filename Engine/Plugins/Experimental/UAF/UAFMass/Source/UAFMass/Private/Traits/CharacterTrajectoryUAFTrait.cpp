// Copyright Epic Games, Inc. All Rights Reserved.

#include "CharacterTrajectoryUAFTrait.h"

#include "Component/AnimNextComponent.h"
#include "Fragments/MassUAFComponentFragment.h"
#include "MassDebugLogging.h"
#include "MassEntityTemplateRegistry.h"
#include "MassEntityView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CharacterTrajectoryUAFTrait)

void UCharacterTrajectoryUAFTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FMassUAFComponentWrapperFragment>();

#if WITH_MASSENTITY_DEBUG
	BuildContext.AddFragment<FMassDebugLogFragment>();
#endif

	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);
	const FConstSharedStruct UAFDataSharedFragment = EntityManager.GetOrCreateConstSharedFragment(UAFData);
	BuildContext.AddConstSharedFragment(UAFDataSharedFragment);

	BuildContext.GetMutableObjectFragmentInitializers().Add([](UObject& Owner, const FMassEntityView& EntityView, const EMassTranslationDirection /*CurrentDirection*/)
		{
			const AActor* AsActor = Cast<AActor>(&Owner);
			UUAFComponent* UAFComponent = AsActor
				? AsActor->FindComponentByClass<UUAFComponent>()
				: Cast<UUAFComponent>(&Owner);

			if (UAFComponent != nullptr)
			{
				FMassUAFComponentWrapperFragment& ComponentFragment = EntityView.GetFragmentData<FMassUAFComponentWrapperFragment>();
				ComponentFragment.Component = UAFComponent;
			}

#if WITH_MASSENTITY_DEBUG
			FMassDebugLogFragment& DebugLogFragment = EntityView.GetFragmentData<FMassDebugLogFragment>();
			DebugLogFragment.LogOwner = AsActor;
#endif
		});
}
