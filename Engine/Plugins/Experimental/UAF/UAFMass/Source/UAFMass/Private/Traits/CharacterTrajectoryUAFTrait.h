// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTraitBase.h"
#include "Fragments/CharacterTrajectoryUAFFragments.h"

#include "CharacterTrajectoryUAFTrait.generated.h"

// This trait adds the UAF component wrapper and configures variable name
// references used by UCharacterTrajectoryToUAFProcessor. It should be used alongside
// UCharacterTrajectoryTrait when UAF integration is needed.
UCLASS(EditInlineNew, CollapseCategories, meta = (DisplayName = "Character Trajectory UAF Setup"))
class UCharacterTrajectoryUAFTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	UPROPERTY(Category = "UAF", EditAnywhere)
	FCharacterTrajectoryUAFData UAFData;
};
