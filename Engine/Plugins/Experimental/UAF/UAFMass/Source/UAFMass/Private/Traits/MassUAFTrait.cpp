// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassUAFTrait.h"

#include "Fragments/MassUAFFragment.h"
#include "MassEntityTemplateRegistry.h"
#include "MassEntityView.h"
#include "MassTranslator.h"
#include "Component/AnimNextComponent.h"
#include "UAF/UAFAssetFactory.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassUAFTrait)

void UMassUAFTrait::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::UAFAssetData)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Asset_DEPRECATED != nullptr && AssetData.IsValid() == false)
		{
			AssetData = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFSystemFactoryAsset>(Asset_DEPRECATED);
		}
		Asset_DEPRECATED = nullptr;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITORONLY_DATA
}

void UMassUAFTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	using namespace UE::UAF;

	FMassUAFFragment& UAFFragment = BuildContext.AddFragment_GetRef<FMassUAFFragment>();
	UAFFragment.Asset = AssetData;
}
