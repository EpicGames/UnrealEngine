// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTraitBase.h"
#include "UAF/UAFAssetData.h"
#include "MassUAFTrait.generated.h"

struct FMassEntityTemplateBuildContext;

UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisplayName = "Mass UAF Trait"))
class UMassUAFTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()
	
	// UObject interface
	virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITORONLY_DATA
	/** The asset that this Mass Entity will run in UAF */
	UE_DEPRECATED(5.8, "Use AssetData instead")
	UPROPERTY()
	TObjectPtr<const UObject> Asset_DEPRECATED = nullptr;
#endif // WITH_EDITORONLY_DATA
	
	/** The asset that this Mass Entity will run in UAF */
	UPROPERTY(EditAnywhere, Category = "Asset", DisplayName = "Asset")
	TInstancedStruct<FUAFSystemFactoryAsset> AssetData;

protected:
	virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};
