// Copyright Epic Games, Inc. All Rights Reserved.
#include "UAFMassModule.h"

#include "CoreMinimal.h"
#include "RigVMCore/RigVMRegistry.h"

#include "Modules/ModuleManager.h"


void FUAFMassModule::StartupModule()
{

}

void FUAFMassModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

IMPLEMENT_MODULE(FUAFMassModule, UAFMass)