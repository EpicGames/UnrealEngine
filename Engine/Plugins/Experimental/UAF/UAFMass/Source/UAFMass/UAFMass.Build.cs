// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UAFMass : ModuleRules
{
	public UAFMass(ReadOnlyTargetRules Target) : base(Target)
	{

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"UAF",
				"MassCommon"
			}
			);


		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"RigVM",
				"Engine",
				"MassSpawner",
				"MassSimulation",
				"MassCore",
				"MassEntity",
				"MassCharacterTrajectory"
			}
			);

		SetupGameplayDebuggerSupport(Target);
	}
}
