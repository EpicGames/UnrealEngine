{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "UAF Mass",
	"Description" : "Mass integration for UAF.",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Plugins":
	[
		{
			"Name": "MassGameplay",
			"Enabled": true
		},
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true
		},
	],
	"Modules" :
	[
		{
			"Name" : "UAFMass",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		}
	]
}
